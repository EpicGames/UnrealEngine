{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Customizable Sequencer Tracks (Experimental)",
	"Description" : "Library that provides a blueprintable track type that can be added to sequencer",
	"Category" : "Runtime",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com/",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "CustomizableSequencerTracks",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "CustomizableSequencerTracksEditor",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		}
	]
}
