// Copyright Epic Games, Inc. All Rights Reserved.


#include "SequencerSectionBP.h"
#include "SequencerTrackBP.h"

#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/BuiltInComponentTypes.h"

#include "SequencerTrackInstanceBP.h"

USequencerSectionBP::USequencerSectionBP(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	// This section must be public as the object animator system needs to reference it and it lives in a different package.
	// Without this flag, object reinstancing will clear out the pointer to the section with FArchiveReplaceOrClearExternalReferences
	SetFlags(RF_Public);
}


void USequencerSectionBP::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	USequencerTrackBP* CustomTrack = GetTypedOuter<USequencerTrackBP>();
	if (CustomTrack->TrackInstanceType.Get())
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

		FGuid ObjectBindingID = Params.GetObjectBindingID();
		OutImportedEntity->AddBuilder(
			FEntityBuilder()
			.Add(BuiltInComponents->TrackInstance, FMovieSceneTrackInstanceComponent{ decltype(FMovieSceneTrackInstanceComponent::Owner)(this), CustomTrack->TrackInstanceType })
			.AddConditional(BuiltInComponents->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid())
			.AddTagConditional(BuiltInComponents->Tags.Root, !ObjectBindingID.IsValid())
		);
	}
}
