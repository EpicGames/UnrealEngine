// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTrackBP.h"

#include "UObject/AssetRegistryTagsContext.h"

UMovieSceneSection* USequencerTrackBP::CreateNewSection()
{
	if (UClass* Class = DefaultSectionType.Get())
	{
		return NewObject<USequencerSectionBP>(this, Class, NAME_None, RF_Transactional);
	}

	for (TSubclassOf<USequencerSectionBP> SupportedSection : SupportedSections)
	{
		if (UClass* Class = SupportedSection.Get())
		{
			return NewObject<USequencerSectionBP>(this, Class, NAME_None, RF_Transactional);
		}
	}

	ensureMsgf(false, TEXT("Track does not have any supported section types. Returning a base class to avoid crashing."));
	return NewObject<UMovieSceneSection>(this, NAME_None, RF_Transactional);
}

void USequencerTrackBP::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void USequencerTrackBP::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);
}
