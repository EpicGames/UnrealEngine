// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTrackInstanceBP.h"
#include "SequencerSectionBP.h"

#include "EntitySystem/MovieSceneInstanceRegistry.h"

#include "Modules/ModuleManager.h"

UObject* USequencerTrackInstanceBP::GetAnimatedObject() const
{
	return Super::GetAnimatedObject();
}

TArray<FSequencerTrackInstanceInput> USequencerTrackInstanceBP::GetInputs() const
{
	using namespace UE::MovieScene;

	TArray<FSequencerTrackInstanceInput> Result;

	TArrayView<const FMovieSceneTrackInstanceInput> InputsView = Super::GetInputs();
	Result.Reserve(InputsView.Num());

	for (const FMovieSceneTrackInstanceInput& Input : InputsView)
	{
		FSequencerTrackInstanceInput ThisInput;
		ThisInput.Section = CastChecked<USequencerSectionBP>(Input.Section);

		const FInstanceRegistry* InstanceRegistry = GetLinker()->GetInstanceRegistry();
		if (ensure(InstanceRegistry->IsHandleValid(Input.InstanceHandle)))
		{
			ThisInput.Context = InstanceRegistry->GetInstance(Input.InstanceHandle).GetContext();
		}

		Result.Add(ThisInput);
	}

	return Result;
}

int32 USequencerTrackInstanceBP::GetNumInputs() const
{
	return GetInputs().Num();
}


FSequencerTrackInstanceInput USequencerTrackInstanceBP::GetInput(int32 Index) const
{
	using namespace UE::MovieScene;

	TArrayView<const FMovieSceneTrackInstanceInput> InputsView = Super::GetInputs();
	if (!InputsView.IsValidIndex(Index))
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Attempting to acces input %d from an array of size %d."), Index, InputsView.Num()), ELogVerbosity::Error);
		return FSequencerTrackInstanceInput();
	}

	FMovieSceneTrackInstanceInput Input = InputsView[Index];

	FSequencerTrackInstanceInput Result;
	Result.Section = CastChecked<USequencerSectionBP>(Input.Section);

	const FInstanceRegistry* InstanceRegistry = GetLinker()->GetInstanceRegistry();
	if (ensure(InstanceRegistry->IsHandleValid(Input.InstanceHandle)))
	{
		Result.Context = InstanceRegistry->GetInstance(Input.InstanceHandle).GetContext();
	}

	return Result;
}

void USequencerTrackInstanceBP::OnInputAdded(const FMovieSceneTrackInstanceInput& InInput)
{
	using namespace UE::MovieScene;

	FSequencerTrackInstanceInput Result;
	Result.Section = CastChecked<USequencerSectionBP>(InInput.Section);

	const FInstanceRegistry* InstanceRegistry = GetLinker()->GetInstanceRegistry();
	if (ensure(InstanceRegistry->IsHandleValid(InInput.InstanceHandle)))
	{
		Result.Context = InstanceRegistry->GetInstance(InInput.InstanceHandle).GetContext();
	}

	K2_OnInputAdded(Result);
}

void USequencerTrackInstanceBP::OnInputRemoved(const FMovieSceneTrackInstanceInput& InInput)
{
	using namespace UE::MovieScene;

	FSequencerTrackInstanceInput Result;
	Result.Section = CastChecked<USequencerSectionBP>(InInput.Section);

	const FInstanceRegistry* InstanceRegistry = GetLinker()->GetInstanceRegistry();
	if (ensure(InstanceRegistry->IsHandleValid(InInput.InstanceHandle)))
	{
		Result.Context = InstanceRegistry->GetInstance(InInput.InstanceHandle).GetContext();
	}

	K2_OnInputRemoved(Result);
}


void USequencerTrackInstanceBP::OnDestroyed()
{
	using namespace UE::MovieScene;

	TArrayView<const FMovieSceneTrackInstanceInput> InputsView = Super::GetInputs();
	for (const FMovieSceneTrackInstanceInput& Input : InputsView)
	{
		FSequencerTrackInstanceInput ThisInput;
		ThisInput.Section = CastChecked<USequencerSectionBP>(Input.Section);

		const FInstanceRegistry* InstanceRegistry = GetLinker()->GetInstanceRegistry();
		if (ensure(InstanceRegistry->IsHandleValid(Input.InstanceHandle)))
		{
			ThisInput.Context = InstanceRegistry->GetInstance(Input.InstanceHandle).GetContext();
		}

		K2_OnInputRemoved(ThisInput);
	}
	K2_OnDestroyed();
}


IMPLEMENT_MODULE(FDefaultModuleImpl, CustomizableSequencerTracks);
