// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSection.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "EntitySystem/BuiltInComponentTypes.h"

#include "SequencerSectionBP.generated.h"


UCLASS(Blueprintable, Abstract, DisplayName=SequencerSection)
class CUSTOMIZABLESEQUENCERTRACKS_API USequencerSectionBP
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
public:

	GENERATED_BODY()

	USequencerSectionBP(const FObjectInitializer& ObjInit);

private:

	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
};
