// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneNameableTrack.h"
#include "SequencerSectionBP.h"
#include "SequencerTrackInstanceBP.h"
#include "Styling/SlateBrush.h"

#include "SequencerTrackBP.generated.h"

class USequencerSectionBP;

UENUM()
enum class ECustomSequencerTrackType
{
	RootTrack,
	ObjectTrack,
};


UCLASS(Blueprintable, Abstract, DisplayName=SequencerTrack)
class CUSTOMIZABLESEQUENCERTRACKS_API USequencerTrackBP
	: public UMovieSceneNameableTrack
{
public:

	GENERATED_BODY()

	UPROPERTY(Category="Sequencer", EditDefaultsOnly, AssetRegistrySearchable)
	bool bSupportsMultipleRows;

	UPROPERTY(Category="Sequencer", EditDefaultsOnly, AssetRegistrySearchable)
	bool bSupportsBlending;

	UPROPERTY(Category="Sequencer", EditDefaultsOnly, AssetRegistrySearchable)
	ECustomSequencerTrackType TrackType;

	UPROPERTY(Category="Sequencer", EditDefaultsOnly, AssetRegistrySearchable, meta=(EditCondition="TrackType==ECustomSequencerTrackType::ObjectTrack"))
	TObjectPtr<UClass> SupportedObjectType;

	UPROPERTY(Category="Sequencer", EditDefaultsOnly, AssetRegistrySearchable)
	TSubclassOf<USequencerSectionBP> DefaultSectionType;

	UPROPERTY(Category="Sequencer", EditDefaultsOnly, AssetRegistrySearchable)
	TArray<TSubclassOf<USequencerSectionBP>> SupportedSections;

	UPROPERTY(Category="Sequencer", EditDefaultsOnly, AssetRegistrySearchable)
	TSubclassOf<USequencerTrackInstanceBP> TrackInstanceType;

	UPROPERTY(Category="Sequencer", EditDefaultsOnly, AssetRegistrySearchable)
	FSlateBrush Icon;

public:

	virtual bool SupportsMultipleRows() const override { return bSupportsMultipleRows; }
	virtual EMovieSceneTrackEasingSupportFlags SupportsEasing(FMovieSceneSupportsEasingParams& Params) const override { return bSupportsBlending ? EMovieSceneTrackEasingSupportFlags::All : EMovieSceneTrackEasingSupportFlags::None; }
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override { return DefaultSectionType == SectionClass || SupportedSections.Contains(SectionClass); }
	virtual UMovieSceneSection* CreateNewSection() override;

	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override { return Sections; }
	virtual void AddSection(UMovieSceneSection& Section) override { Sections.AddUnique(&Section); }
	virtual void RemoveSection(UMovieSceneSection& Section) override { Sections.Remove(&Section); }
	virtual void RemoveSectionAt(int32 SectionIndex) override { Sections.RemoveAt(SectionIndex); }
	virtual bool HasSection(const UMovieSceneSection& Section) const override { return Sections.Contains(&Section); }
	virtual bool IsEmpty() const override { return Sections.Num() == 0; }

	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;

private:

	UPROPERTY(Instanced)
	TArray<TObjectPtr<UMovieSceneSection>> Sections;
};