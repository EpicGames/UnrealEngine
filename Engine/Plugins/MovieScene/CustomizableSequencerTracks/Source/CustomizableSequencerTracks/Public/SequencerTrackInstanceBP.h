// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneNameableTrack.h"
#include "SequencerSectionBP.h"
#include "EntitySystem/TrackInstance/MovieSceneTrackInstance.h"
#include "SequencerTrackInstanceBP.generated.h"

class USequencerSectionBP;

USTRUCT(BlueprintType)
struct FSequencerTrackInstanceInput
{
	GENERATED_BODY()

	FSequencerTrackInstanceInput()
		: Section(nullptr)
		, Context(FMovieSceneEvaluationRange(0, FFrameRate()))
	{}

	UPROPERTY(Category="Sequencer", BlueprintReadOnly)
	TObjectPtr<USequencerSectionBP> Section;

	FMovieSceneContext Context;
};


UCLASS(Blueprintable, Abstract, DisplayName=SequencerTrackInstance)
class CUSTOMIZABLESEQUENCERTRACKS_API USequencerTrackInstanceBP
	: public UMovieSceneTrackInstance
{
public:

	GENERATED_BODY()

	/*~ Implementable interface */

	UFUNCTION(Category="Sequencer", DisplayName="OnInitialize", BlueprintImplementableEvent, meta=(CallInEditor=true))
	void K2_OnInitialize();

	UFUNCTION(Category="Sequencer", DisplayName="OnUpdate", BlueprintImplementableEvent, meta=(CallInEditor=true))
	void K2_OnUpdate();

	UFUNCTION(Category="Sequencer", DisplayName="OnBeginUpdateInputs", BlueprintImplementableEvent, meta=(CallInEditor=true))
	void K2_OnBeginUpdateInputs();

	UFUNCTION(Category="Sequencer", DisplayName="OnInputAdded", BlueprintImplementableEvent, meta=(CallInEditor=true))
	void K2_OnInputAdded(FSequencerTrackInstanceInput Input);

	UFUNCTION(Category="Sequencer", DisplayName="OnInputRemoved", BlueprintImplementableEvent, meta=(CallInEditor=true))
	void K2_OnInputRemoved(FSequencerTrackInstanceInput Input);

	UFUNCTION(Category="Sequencer", DisplayName="OnEndUpdateInputs", BlueprintImplementableEvent, meta=(CallInEditor=true))
	void K2_OnEndUpdateInputs();

	UFUNCTION(Category="Sequencer", DisplayName="OnDestroyed", BlueprintImplementableEvent, meta=(CallInEditor=true))
	void K2_OnDestroyed();

public:

	UFUNCTION(Category="Sequencer", BlueprintCallable)
	UObject* GetAnimatedObject() const;

	UFUNCTION(Category="Sequencer", BlueprintCallable)
	TArray<FSequencerTrackInstanceInput> GetInputs() const;

	UFUNCTION(Category="Sequencer", BlueprintCallable)
	int32 GetNumInputs() const;

	UFUNCTION(Category="Sequencer", BlueprintCallable, BlueprintPure=false)
	FSequencerTrackInstanceInput GetInput(int32 Index) const;

private:

	virtual void OnInitialize() override { K2_OnInitialize(); }

	virtual void OnAnimate() override { K2_OnUpdate(); }

	virtual void OnBeginUpdateInputs() override { K2_OnBeginUpdateInputs(); }

	virtual void OnInputAdded(const FMovieSceneTrackInstanceInput& InInput) override;

	virtual void OnInputRemoved(const FMovieSceneTrackInstanceInput& InInput) override;

	virtual void OnEndUpdateInputs() override { K2_OnEndUpdateInputs(); }

	virtual void OnDestroyed() override;
};

