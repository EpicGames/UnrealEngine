// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CustomizableSequencerTracksEditor : ModuleRules
{
	public CustomizableSequencerTracksEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
				"Core",
				"CoreUObject",
				"CustomizableSequencerTracks",
				"EditorFramework",
				"MovieScene",
				"SequencerCore",
				"Sequencer",
				"SlateCore",
				"Slate",
				"UnrealEd",
			}
		);

		ShortName = "CustomSeqTrEd";
	}
}
