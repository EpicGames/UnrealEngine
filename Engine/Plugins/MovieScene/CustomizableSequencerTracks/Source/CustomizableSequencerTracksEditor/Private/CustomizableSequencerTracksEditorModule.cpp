// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "ISequencerModule.h"
#include "SequencerTrackBPEditor.h"
#include "CustomizableSequencerTracksStyle.h"

class FCustomizableSequencerTracksEditorModule : public IModuleInterface
{
	FDelegateHandle CustomTrackEditorHandle;

	virtual void StartupModule() override
	{
		ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
		CustomTrackEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FSequencerTrackBPEditor::CreateTrackEditor));

		FCustomizableSequencerTracksStyle::Get();
	}
	virtual void ShutdownModule() override
	{
		if (ISequencerModule* SequencerModule = FModuleManager::Get().GetModulePtr<ISequencerModule>("Sequencer"))
		{
			SequencerModule->UnRegisterTrackEditor(CustomTrackEditorHandle);
		}
	}
};

IMPLEMENT_MODULE(FCustomizableSequencerTracksEditorModule, CustomizableSequencerTracksEditor);


