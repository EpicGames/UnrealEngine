// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomizableSequencerTracksStyle.h"
#include "SequencerTrackBP.h"

#include "Styling/SlateStyleRegistry.h"

FName FCustomizableSequencerTracksStyle::StyleName("CustomizableSequencerTracksStyle");

FCustomizableSequencerTracksStyle::FCustomizableSequencerTracksStyle()
	: FSlateStyleSet(StyleName)
{
	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FCustomizableSequencerTracksStyle::~FCustomizableSequencerTracksStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FCustomizableSequencerTracksStyle& FCustomizableSequencerTracksStyle::Get()
{
	static FCustomizableSequencerTracksStyle Inst;
	return Inst;
}

void FCustomizableSequencerTracksStyle::RegisterNewTrackType(TSubclassOf<USequencerTrackBP> TrackType)
{
	UClass* Class = TrackType.Get();
	if (Class)
	{
		USequencerTrackBP* CDO = Cast<USequencerTrackBP>(Class->GetDefaultObject());
		Set(Class->GetFName(), new FSlateBrush(CDO->Icon));
	}
}
