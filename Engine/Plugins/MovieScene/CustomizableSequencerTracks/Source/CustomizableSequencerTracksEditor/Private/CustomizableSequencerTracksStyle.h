// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class USequencerTrackBP;

class FCustomizableSequencerTracksStyle : public FSlateStyleSet
{
public:

	static FName StyleName;

	/** Access the singleton instance for this style set */
	static FCustomizableSequencerTracksStyle& Get();

	void RegisterNewTrackType(TSubclassOf<USequencerTrackBP> TrackType);

private:

	FCustomizableSequencerTracksStyle();
	~FCustomizableSequencerTracksStyle();
};
