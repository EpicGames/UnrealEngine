// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTrackBPEditor.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "Blueprint/BlueprintSupport.h"
#include "SequencerTrackBP.h"
#include "ISequencerSection.h"
#include "SequencerSectionPainter.h"
#include "Misc/PackageName.h"
#include "MVVM/Views/ViewUtilities.h"
#include "CustomizableSequencerTracksStyle.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"
#include "Misc/ScopedSlowTask.h"
#include "Engine/Blueprint.h"

#include "ScopedTransaction.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SequencerTrackBPEditor"

namespace
{
	static TArray<FAssetData> DiscoverCustomTrackTypes()
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		TArray<FAssetData> ValidClasses;
		{
			FString DesiredClassName = FString::Printf(TEXT("/Script/CoreUObject.Class'%s'"), *USequencerTrackBP::StaticClass()->GetPathName());

			FARFilter Filter;
			Filter.TagsAndValues.Add(FBlueprintTags::NativeParentClassPath, MoveTemp(DesiredClassName));
			AssetRegistryModule.Get().GetAssets(Filter, ValidClasses);
		}

		TSet<FName> ExistingPaths;
		for (const FAssetData& Asset : ValidClasses)
		{
			FAssetDataTagMapSharedView::FFindTagResult GeneratedClassPathTag = Asset.TagsAndValues.FindTag(FBlueprintTags::GeneratedClassPath);
			if (GeneratedClassPathTag.IsSet())
			{
				FString ObjectPath = FPackageName::ExportTextPathToObjectPath(GeneratedClassPathTag.GetValue());
				ExistingPaths.Add(*ObjectPath);
			}
		}

		// Check loaded classes
		UClass* CustomSequencerTrackClass = USequencerTrackBP::StaticClass();
		for (TObjectIterator<UClass> ClassIterator; ClassIterator; ++ClassIterator)
		{
			UClass* Class = *ClassIterator;
			if (Class->IsChildOf(CustomSequencerTrackClass) && !Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists) && Class->GetAuthoritativeClass() == Class && Class != CustomSequencerTrackClass)
			{
				if (!ExistingPaths.Contains(*Class->GetPathName()))
				{
					ValidClasses.Add(*ClassIterator);
				}
			}
		}

		return ValidClasses;
	}

	UClass* LoadClassFromAssetData(const FAssetData& AssetData)
	{
		UObject* LoadedAsset = AssetData.FastGetAsset();
		if (!LoadedAsset)
		{
			FScopedSlowTask SlowTask(1.f, FText::Format(LOCTEXT("LoadingClass", "Loading asset {0}"), FText::FromName(AssetData.AssetName)));
			SlowTask.MakeDialogDelayed(1.f);

			LoadedAsset = AssetData.GetAsset();
		}

		if (!LoadedAsset)
		{
			return nullptr;
		}

		if (UClass* DirectClass = Cast<UClass>(LoadedAsset))
		{
			return DirectClass;
		}
		else if (UBlueprint* Blueprint = Cast<UBlueprint>(LoadedAsset))
		{
			return Blueprint->GeneratedClass;
		}
		return nullptr;
	}
}

struct FCustomSection : public ISequencerSection, public FGCObject
{
	FCustomSection(UMovieSceneSection* InSection)
		: Section(InSection)
	{}

	virtual UMovieSceneSection* GetSectionObject() override
	{
		return Section;
	}

	virtual int32 OnPaintSection(FSequencerSectionPainter& Painter) const override
	{
		return Painter.PaintSectionBackground();
	}

	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override
	{
		Collector.AddReferencedObject(Section);
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FCustomSection");
	}

	TObjectPtr<UMovieSceneSection> Section;
};


FSequencerTrackBPEditor::FSequencerTrackBPEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{}


TSharedRef<ISequencerSection> FSequencerTrackBPEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	return MakeShared<FCustomSection>(&SectionObject);
}


void FSequencerTrackBPEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	FCustomizableSequencerTracksStyle& StyleSet = FCustomizableSequencerTracksStyle::Get();

	TArray<FAssetData> TrackTypes = DiscoverCustomTrackTypes();

	// Prune tracks that do not support this object class, or are root tracks
	FName TrackTypeTagName = GET_MEMBER_NAME_CHECKED(USequencerTrackBP, TrackType);
	for (const FAssetData& Asset : TrackTypes)
	{
		FAssetDataTagMapSharedView::FFindTagResult TrackTypeTag = Asset.TagsAndValues.FindTag(TrackTypeTagName);
		if (TrackTypeTag.IsSet() && TrackTypeTag.GetValue() == TEXT("RootTrack"))
		{
			continue;
		}

		// We have to try and load the class to check whether it is compatible
		UClass* ThisClass = LoadClassFromAssetData(Asset);
		if (!ThisClass)
		{
			continue;
		}

		USequencerTrackBP* CDO = Cast<USequencerTrackBP>(ThisClass->GetDefaultObject());

		if (ObjectClass && CDO->SupportedObjectType && !ObjectClass->IsChildOf(CDO->SupportedObjectType))
		{
			continue;
		}

		StyleSet.RegisterNewTrackType(ThisClass);

		MenuBuilder.AddMenuEntry(
			FText::FromName(Asset.AssetName),
			FText(),
			FSlateIcon(StyleSet.GetStyleSetName(), ThisClass->GetFName()),
			FUIAction(
				FExecuteAction::CreateSP(this, &FSequencerTrackBPEditor::AddNewObjectBindingTrack, Asset, ObjectBindings)
			)
		);
	}
}

void FSequencerTrackBPEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	FCustomizableSequencerTracksStyle& StyleSet = FCustomizableSequencerTracksStyle::Get();

	TArray<FAssetData> TrackTypes = DiscoverCustomTrackTypes();

	// Only show tracks that can be root tracks
	FName TrackTypeTagName = GET_MEMBER_NAME_CHECKED(USequencerTrackBP, TrackType);
	for (const FAssetData& Asset : TrackTypes)
	{
		FAssetDataTagMapSharedView::FFindTagResult TrackTypeTag = Asset.TagsAndValues.FindTag(TrackTypeTagName);

		if (TrackTypeTag.IsSet() && TrackTypeTag.GetValue() == TEXT("RootTrack"))
		{
			UClass* ThisClass = LoadClassFromAssetData(Asset);
			if (!ThisClass)
			{
				continue;
			}

			StyleSet.RegisterNewTrackType(ThisClass);

			MenuBuilder.AddMenuEntry(
				FText::FromName(Asset.AssetName),
				FText(),
				FSlateIcon(StyleSet.GetStyleSetName(), ThisClass->GetFName()),
				FUIAction(
					FExecuteAction::CreateSP(this, &FSequencerTrackBPEditor::AddNewTrack, Asset)
				)
			);
		}

	}
}

TSharedPtr<SWidget> FSequencerTrackBPEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	USequencerTrackBP* CustomTrack = Cast<USequencerTrackBP>(Track);
	if (!CustomTrack)
	{
		return nullptr;
	}

	if (CustomTrack->SupportedSections.Num() > 0)
	{
		auto SubMenuCallback = [this, CustomTrack]() -> TSharedRef<SWidget>
		{
			FMenuBuilder MenuBuilder(true, nullptr);

			this->MakeMenuEntry(MenuBuilder, CustomTrack, CustomTrack->DefaultSectionType);
			for (TSubclassOf<USequencerSectionBP> ClassType : CustomTrack->SupportedSections)
			{
				this->MakeMenuEntry(MenuBuilder, CustomTrack, ClassType);
			}

			return MenuBuilder.MakeWidget();
		};

		return UE::Sequencer::MakeAddButton(LOCTEXT("AddSectionText", "Section"), FOnGetContent::CreateLambda(SubMenuCallback), Params.ViewModel);
	}


	UClass* Class = CustomTrack->DefaultSectionType.Get();
	if (Class)
	{
		TWeakPtr<ISequencer> WeakSequencer = GetSequencer();
		FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 8);

		return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.OnClicked_Lambda([this, CustomTrack]() -> FReply { this->CreateNewSection(CustomTrack, CustomTrack->DefaultSectionType); return FReply::Handled(); })
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ForegroundColor(FSlateColor::UseForeground())
			.IsEnabled_Lambda([=]() { return WeakSequencer.IsValid() ? !WeakSequencer.Pin()->IsReadOnly() : false; })
			.ContentPadding(FMargin(5, 2))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0,0,2,0))
				[
					SNew(SImage)
					.ColorAndOpacity( FSlateColor::UseForeground() )
					.Image(FAppStyle::GetBrush("Plus"))
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Visibility_Lambda([Hovered = Params.NodeIsHovered]() -> EVisibility { return Hovered.Get() ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed; })
					.Text(Class->GetDisplayNameText())
					.Font(SmallLayoutFont)
					.ColorAndOpacity( FSlateColor::UseForeground() )
				]
			]
		];
	}

	return nullptr;
}

void FSequencerTrackBPEditor::MakeMenuEntry(FMenuBuilder& MenuBuilder, USequencerTrackBP* Track, TSubclassOf<USequencerSectionBP> ClassType)
{
	UClass* Class = ClassType.Get();
	if (Class)
	{
		MenuBuilder.AddMenuEntry(Class->GetDisplayNameText(), Class->GetToolTipText(), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &FSequencerTrackBPEditor::CreateNewSection, Track, ClassType)));
	}
}

void FSequencerTrackBPEditor::CreateNewSection(USequencerTrackBP* Track, TSubclassOf<USequencerSectionBP> ClassType)
{
	TSharedPtr<ISequencer> SequencerPin = GetSequencer();
	UClass*                Class     = ClassType.Get();

	if (Class && SequencerPin)
	{
		FScopedTransaction Transaction(FText::Format(LOCTEXT("AddCustomSection_Transaction", "Add New Section From Class {0}"), FText::FromName(Class->GetFName())));

		Track->Modify();
			
		USequencerSectionBP* NewSection = NewObject<USequencerSectionBP>(Track, Class, NAME_None, RF_Transactional);

		FQualifiedFrameTime CurrentTime = SequencerPin->GetLocalTime();

		const FFrameNumber Duration = (5.f * CurrentTime.Rate).FrameNumber;
		NewSection->SetRange(TRange<FFrameNumber>(CurrentTime.Time.FrameNumber, CurrentTime.Time.FrameNumber + Duration));
		NewSection->InitialPlacement(Track->GetAllSections(), CurrentTime.Time.FrameNumber, Duration.Value, Track->SupportsMultipleRows());

		Track->AddSection(*NewSection);

		SequencerPin->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
	}
}

bool FSequencerTrackBPEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type->IsChildOf(USequencerTrackBP::StaticClass());
}

void FSequencerTrackBPEditor::AddNewTrack(FAssetData AssetData)
{
	UMovieScene* MovieScene = GetFocusedMovieScene();
	if (MovieScene == nullptr || MovieScene->IsReadOnly())
	{
		return;
	}

	UClass* ClassToAdd = LoadClassFromAssetData(AssetData);
	if (!ensure(ClassToAdd && ClassToAdd->IsChildOf(USequencerTrackBP::StaticClass())))
	{
		// @todo: notification error
		return;
	}

	const FScopedTransaction Transaction(FText::Format(LOCTEXT("AddCustomTrack_Transaction", "Add Track {0}"), FText::FromName(ClassToAdd->GetFName())));

	MovieScene->Modify();

	USequencerTrackBP* CustomTrack = CastChecked<USequencerTrackBP>(MovieScene->AddTrack(ClassToAdd));
	CreateNewSection(CustomTrack, CustomTrack->DefaultSectionType);

	if (GetSequencer().IsValid())
	{
		GetSequencer()->OnAddTrack(CustomTrack, FGuid());
	}
}

void FSequencerTrackBPEditor::AddNewObjectBindingTrack(FAssetData AssetData, TArray<FGuid> InObjectBindings)
{
	UMovieScene* MovieScene = GetFocusedMovieScene();
	if (MovieScene == nullptr || MovieScene->IsReadOnly())
	{
		return;
	}

	UClass* ClassToAdd = LoadClassFromAssetData(AssetData);
	if (!ensure(ClassToAdd && ClassToAdd->IsChildOf(USequencerTrackBP::StaticClass())))
	{
		// @todo: notification error
		return;
	}

	const FScopedTransaction Transaction(FText::Format(LOCTEXT("AddCustomObjectTrack_Transaction", "Add Object Track {0}"), FText::FromName(ClassToAdd->GetFName())));

	MovieScene->Modify();

	for (const FGuid& ObjectBindingID : InObjectBindings)
	{
		USequencerTrackBP* CustomTrack = CastChecked<USequencerTrackBP>(MovieScene->AddTrack(ClassToAdd, ObjectBindingID));
		CreateNewSection(CustomTrack, CustomTrack->DefaultSectionType);

		if (GetSequencer().IsValid())
		{
			GetSequencer()->OnAddTrack(CustomTrack, FGuid());
		}
	}
}

#undef LOCTEXT_NAMESPACE
