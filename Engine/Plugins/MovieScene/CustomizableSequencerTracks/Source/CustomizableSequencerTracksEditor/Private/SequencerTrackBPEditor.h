// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneTrackEditor.h"

struct FAssetData;

class USequencerTrackBP;
class USequencerSectionBP;


/**
* A track editor that handles all types of USequencerTrackBP
*/
class FSequencerTrackBPEditor
	: public FMovieSceneTrackEditor
{
public:

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
	{
		return MakeShared<FSequencerTrackBPEditor>(InSequencer);
	}

public:

	FSequencerTrackBPEditor(TSharedRef<ISequencer> InSequencer);

private:

	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;

private:

	void AddNewTrack(FAssetData AssetData);

	void AddNewObjectBindingTrack(FAssetData AssetData, TArray<FGuid> InObjectBindings);

	void MakeMenuEntry(FMenuBuilder& MenuBuilder, USequencerTrackBP* Track, TSubclassOf<USequencerSectionBP> ClassType);

	void CreateNewSection(USequencerTrackBP* Track, TSubclassOf<USequencerSectionBP> ClassType);
};
